{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Niagara MRQ Support",
	"Description": "Contains a data interface that can be used to read Movie Render Queue information in Niagara simulations.",
	"Category": "FX",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "NiagaraMRQ",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
	],
	"Plugins": [
		{
			"Name": "MovieRenderPipeline",
			"Enabled": true
		},
		{
			"Name": "Niagara",
			"Enabled": true
		}
	]
}