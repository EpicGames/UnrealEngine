// Copyright Epic Games, Inc. All Rights Reserved.

int		{ParameterName}_Active;
int		{ParameterName}_TemporalSampleCount;
int		{ParameterName}_TemporalSampleIndex;
float	{ParameterName}_SequenceFPS;

void GetMRQInfo_{ParameterName}(out bool bActive, out int TemporalSampleCount, out int TemporalSampleIndex, out float SequenceFPS)
{
	bActive					= {ParameterName}_Active != 0;
	TemporalSampleCount		= {ParameterName}_TemporalSampleCount;
	TemporalSampleIndex		= {ParameterName}_TemporalSampleIndex;
	SequenceFPS				= {ParameterName}_SequenceFPS;
}
