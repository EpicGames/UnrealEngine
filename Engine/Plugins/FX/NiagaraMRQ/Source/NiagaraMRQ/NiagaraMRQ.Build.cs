// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NiagaraMRQ : ModuleRules
	{
		public NiagaraMRQ(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(new[] { "Core" });

			PrivateDependencyModuleNames.AddRange(
				new[]
				{
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"Projects",
					"MovieRenderPipelineCore",
				
					// Data interface dependencies
					"Niagara",
					"NiagaraCore",
					"VectorVM",
					"RenderCore",
					"RHI"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
