// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraDataInterfaceMRQ.h"
#include "NiagaraCompileHashVisitor.h"
#include "NiagaraShaderParametersBuilder.h"
#include "NiagaraSystemInstance.h"

#include "Modules/ModuleManager.h"
#include "MovieRenderPipelineCoreModule.h"

#define LOCTEXT_NAMESPACE "NiagaraDataInterfaceMRQ"

namespace NDIMRQLocal
{
	static const TCHAR* TemplateShaderFile = TEXT("/Plugin/NiagaraMRQ/Private/NiagaraDataInterfaceMRQ.ush");

	const FName NAME_GetMRQInfo(TEXT("GetMRQInfo"));

	struct FNDIInstanceData
	{
		bool		bActive = 0;
		int32		TemporalSampleCount = 1;
		int32		TemporalSampleIndex = 0;
		float		SequenceFPS = 1.0f / 30.0f;
	};

	struct FNDIProxy : public FNiagaraDataInterfaceProxy
	{
		virtual int32 PerInstanceDataPassedToRenderThreadSize() const override { return sizeof(FNDIInstanceData); }

		static void ProvidePerInstanceDataForRenderThread(void* InDataToRenderThread, void* InPerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
		{
			FNDIInstanceData* DataToRenderThread = new (InDataToRenderThread) FNDIInstanceData();
			*DataToRenderThread = *static_cast<const FNDIInstanceData*>(InPerInstanceData);
		}

		virtual void ConsumePerInstanceDataFromGameThread(void* InDataFromGameThread, const FNiagaraSystemInstanceID& InstanceID) override
		{
			const FNDIInstanceData* DataFromGameThread = static_cast<const FNDIInstanceData*>(InDataFromGameThread);
			FNDIInstanceData& InstanceData_RT = SystemInstancesToInstanceData_RT.FindOrAdd(InstanceID);
			InstanceData_RT = *DataFromGameThread;
			DataFromGameThread->~FNDIInstanceData();
		}

		TMap<FNiagaraSystemInstanceID, FNDIInstanceData> SystemInstancesToInstanceData_RT;
	};

	void VMGetMRQInfo(FVectorVMExternalFunctionContext& Context)
	{
		VectorVM::FUserPtrHandler<FNDIInstanceData> InstanceData(Context);
		FNDIOutputParam<bool>	OutActive(Context);
		FNDIOutputParam<int32>	OutTemporalSampleCount(Context);
		FNDIOutputParam<int32>	OutTemporalSampleIndex(Context);
		FNDIOutputParam<float>	OutSequenceFPS(Context);

		for (int32 i = 0; i < Context.GetNumInstances(); ++i)
		{
			OutActive.SetAndAdvance(InstanceData->bActive);
			OutTemporalSampleCount.SetAndAdvance(InstanceData->TemporalSampleCount);
			OutTemporalSampleIndex.SetAndAdvance(InstanceData->TemporalSampleIndex);
			OutSequenceFPS.SetAndAdvance(InstanceData->SequenceFPS);
		}
	}
}

UNiagaraDataInterfaceMRQ::UNiagaraDataInterfaceMRQ(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
	using namespace NDIMRQLocal;
	Proxy.Reset(new FNDIProxy());
}

void UNiagaraDataInterfaceMRQ::PostInitProperties()
{
	Super::PostInitProperties();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		ENiagaraTypeRegistryFlags Flags = ENiagaraTypeRegistryFlags::AllowAnyVariable | ENiagaraTypeRegistryFlags::AllowParameter;
		FNiagaraTypeRegistry::Register(FNiagaraTypeDefinition(GetClass()), Flags);
	}
}

#if WITH_EDITORONLY_DATA
void UNiagaraDataInterfaceMRQ::GetFunctionsInternal(TArray<FNiagaraFunctionSignature>& OutFunctions) const
{
	using namespace NDIMRQLocal;

	FNiagaraFunctionSignature& Sig = OutFunctions.AddDefaulted_GetRef();
	Sig.Name = NAME_GetMRQInfo;
	Sig.SetDescription(LOCTEXT("GetMRQInfoDesc", "Returns information from MRQ and if it's running or not."));
	Sig.bMemberFunction = true;
	Sig.AddInput(FNiagaraVariable(FNiagaraTypeDefinition(GetClass()), TEXT("MRQInterface")));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetBoolDef(), TEXT("IsActive")), LOCTEXT("IsActiveDesc", "True if we are simulating the frame inside MRQ"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("TemporalSampleCount")), LOCTEXT("TemporalSampleCountDesc", "Temporal sample count we are rendering with"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetIntDef(), TEXT("TemporalSampleIndex")), LOCTEXT("TemporalSampleIndexDesc", "Temporal sample we are simulating this frame for"));
	Sig.AddOutput(FNiagaraVariable(FNiagaraTypeDefinition::GetFloatDef(), TEXT("SequenceFPS")), LOCTEXT("SequenceFPSDesc", "The source sequence frames per second"));
}
#endif

void UNiagaraDataInterfaceMRQ::GetVMExternalFunction(const FVMExternalFunctionBindingInfo& BindingInfo, void* InstanceData, FVMExternalFunction& OutFunc)
{
	using namespace NDIMRQLocal;
	if (BindingInfo.Name == NAME_GetMRQInfo)
	{
		OutFunc = FVMExternalFunction::CreateStatic(VMGetMRQInfo);
	}
}

#if WITH_EDITORONLY_DATA
bool UNiagaraDataInterfaceMRQ::AppendCompileHash(FNiagaraCompileHashVisitor* InVisitor) const
{
	using namespace NDIMRQLocal;
	bool bSuccess = Super::AppendCompileHash(InVisitor);
	bSuccess &= InVisitor->UpdateShaderFile(TemplateShaderFile);
	bSuccess &= InVisitor->UpdateShaderParameters<FShaderParameters>();
	return bSuccess;
}

bool UNiagaraDataInterfaceMRQ::GetFunctionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, const FNiagaraDataInterfaceGeneratedFunction& FunctionInfo, int FunctionInstanceIndex, FString& OutHLSL)
{
	using namespace NDIMRQLocal;
	return FunctionInfo.DefinitionName == NAME_GetMRQInfo;
}

void UNiagaraDataInterfaceMRQ::GetParameterDefinitionHLSL(const FNiagaraDataInterfaceGPUParamInfo& ParamInfo, FString& OutHLSL)
{
	using namespace NDIMRQLocal;
	const TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{TEXT("ParameterName"),	ParamInfo.DataInterfaceHLSLSymbol},
	};
	AppendTemplateHLSL(OutHLSL, TemplateShaderFile, TemplateArgs);
}
#endif

void UNiagaraDataInterfaceMRQ::BuildShaderParameters(FNiagaraShaderParametersBuilder& ShaderParametersBuilder) const
{
	ShaderParametersBuilder.AddNestedStruct<FShaderParameters>();
}

void UNiagaraDataInterfaceMRQ::SetShaderParameters(const FNiagaraDataInterfaceSetShaderParametersContext& Context) const
{
	using namespace NDIMRQLocal;
	FNDIProxy& DIProxy = Context.GetProxy<FNDIProxy>();
	FNDIInstanceData& InstanceData = DIProxy.SystemInstancesToInstanceData_RT.FindChecked(Context.GetSystemInstanceID());

	FShaderParameters* ShaderParameters		= Context.GetParameterNestedStruct<FShaderParameters>();
	ShaderParameters->Active				= InstanceData.bActive ? 1 : 0;
	ShaderParameters->TemporalSampleCount	= InstanceData.TemporalSampleCount;
	ShaderParameters->TemporalSampleIndex	= InstanceData.TemporalSampleIndex;
	ShaderParameters->SequenceFPS			= InstanceData.SequenceFPS;
}

bool UNiagaraDataInterfaceMRQ::InitPerInstanceData(void* InPerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIMRQLocal;
	FNDIInstanceData* InstanceData = new (InPerInstanceData) FNDIInstanceData();
	return true;
}

void UNiagaraDataInterfaceMRQ::DestroyPerInstanceData(void* InPerInstanceData, FNiagaraSystemInstance* SystemInstance)
{
	using namespace NDIMRQLocal;
	FNDIInstanceData* InstanceData = reinterpret_cast<FNDIInstanceData*>(InPerInstanceData);
	InstanceData->~FNDIInstanceData();

	ENQUEUE_RENDER_COMMAND(RemoveProxy)
	(
		[RT_Proxy=GetProxyAs<FNDIProxy>(), InstanceID=SystemInstance->GetId()](FRHICommandListImmediate& CmdList)
		{
			RT_Proxy->SystemInstancesToInstanceData_RT.Remove(InstanceID);
		}
	);
}

int32 UNiagaraDataInterfaceMRQ::PerInstanceDataSize() const
{
	using namespace NDIMRQLocal;
	return sizeof(FNDIInstanceData);
}

bool UNiagaraDataInterfaceMRQ::PerInstanceTick(void* InPerInstanceData, FNiagaraSystemInstance* SystemInstance, float DeltaSeconds)
{
	using namespace NDIMRQLocal;

	FNDIInstanceData* InstanceData = reinterpret_cast<FNDIInstanceData*>(InPerInstanceData);

	FMoviePipelineLightweightTickInfo MRQTickInfo;
	if (FMovieRenderPipelineCoreModule* MRQModule = FModuleManager::Get().GetModulePtr<FMovieRenderPipelineCoreModule>("MovieRenderPipelineCore"))
	{
		MRQTickInfo = MRQModule->GetTickInfo();
	}
	InstanceData->bActive = MRQTickInfo.bIsActive;
	InstanceData->TemporalSampleCount = MRQTickInfo.TemporalSampleCount;
	InstanceData->TemporalSampleIndex = MRQTickInfo.TemporalSampleIndex;
	InstanceData->SequenceFPS = float(MRQTickInfo.SequenceFPS);

	return false;
}

void UNiagaraDataInterfaceMRQ::ProvidePerInstanceDataForRenderThread(void* DataForRenderThread, void* PerInstanceData, const FNiagaraSystemInstanceID& SystemInstance)
{
	using namespace NDIMRQLocal;
	FNDIProxy::ProvidePerInstanceDataForRenderThread(DataForRenderThread, PerInstanceData, SystemInstance);
}

#undef LOCTEXT_NAMESPACE
