// Copyright Epic Games, Inc. All Rights Reserved.

#include "NiagaraMRQModule.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"
#include "Interfaces/IPluginManager.h"

#define LOCTEXT_NAMESPACE "FNiagaraMRQModule"

void FNiagaraMRQModule::StartupModule()
{
	// map the shader dir so we can use it in the data interface
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("NiagaraMRQ"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/NiagaraMRQ"), PluginShaderDir);
}

void FNiagaraMRQModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FNiagaraMRQModule, NiagaraMRQ)
